// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LauncherCheck : ModuleRules
{
	public LauncherCheck(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"HTTP",
			}
		);

		if (Target.bUseLauncherChecks)
		{
			PublicDefinitions.Add("WITH_LAUNCHERCHECK=1");
			PublicDependencyModuleNames.Add("LauncherPlatform");
		}
        else
        {
            PublicDefinitions.Add("WITH_LAUNCHERCHECK=0");
        }
    }
}
