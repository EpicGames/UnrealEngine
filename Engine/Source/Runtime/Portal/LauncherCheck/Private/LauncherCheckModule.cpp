// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "HAL/PlatformProcess.h"
#include "ILauncherCheckModule.h"

#if defined(WITH_LAUNCHERCHECK) && WITH_LAUNCHERCHECK

#include "GenericPlatform/GenericPlatformHttp.h"
#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"


/**
 * Log categories for LauncherCheck module
 */
DEFINE_LOG_CATEGORY(LogLauncherCheck);


/**
 * Implements the Launcher Check module.
 */
class FLauncherCheckModule
	: public ILauncherCheckModule
{
public:

	/*
	* Check to see if this module should perform any checks or not
	* @return true, if it should
	*/
	bool IsEnabled() const
	{
		return FParse::Param(FCommandLine::Get(), TEXT("NoEpicPortal")) == false && FParse::Param(FCommandLine::Get(), TEXT("q")) == false;
	}

public:

	//~ ILauncherCheckModule interface

	virtual bool WasRanFromLauncher() const override
	{
		// Check for the presence of a specific param that's passed from the Launcher to the games
		// when we want to make sure we've come from the Launcher
		return !IsEnabled() || FParse::Param(FCommandLine::Get(), TEXT("EpicPortal"));
	}

	virtual bool RunLauncher(ELauncherAction Action, FString Payload = FString()) const override
	{
		ILauncherPlatform* LauncherPlatform = FLauncherPlatformModule::Get();
		if (LauncherPlatform != nullptr)
		{
			// Construct a url to tell the launcher of this app and what we want to do with it
			FOpenLauncherOptions LauncherOptions;
			LauncherOptions.LauncherRelativeUrl = TEXT("apps");
			LauncherOptions.LauncherRelativeUrl /= FLauncherMisc::GetEncodedExePath();
			switch (Action)
			{
			case ELauncherAction::AppLaunch:
				LauncherOptions.LauncherRelativeUrl += TEXT("?action=launch");
				break;
			case ELauncherAction::AppUpdateCheck:
				LauncherOptions.LauncherRelativeUrl += TEXT("?action=updatecheck");
				break;
			case ELauncherAction::AppInstaller:
				LauncherOptions.LauncherRelativeUrl += TEXT("?action=installer");
				break;
			case ELauncherAction::AppVerifyInstallation:
				LauncherOptions.LauncherRelativeUrl += TEXT("?action=verify");
				break;
			};

			// If our payload starts with the correct encoded character, then append the string
			if (Payload.StartsWith(TEXT("%26")))
			{
				LauncherOptions.LauncherRelativeUrl.Append(MoveTemp(Payload));
			}
			return LauncherPlatform->OpenLauncher(LauncherOptions);
		}
		return false;
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		FLauncherPlatformModule::Get();
	}

	virtual void ShutdownModule() override { }
};


#else //WITH_LAUNCHERCHECK


class FLauncherCheckModule
	: public ILauncherCheckModule
{
public:

	virtual bool WasRanFromLauncher() const override
	{
		return true;
	}

	virtual bool RunLauncher(ELauncherAction Action, FString Payload = FString()) const override
	{
		return false;
	}
};

#endif //WITH_LAUNCHERCHECK


IMPLEMENT_MODULE(FLauncherCheckModule, LauncherCheck);
