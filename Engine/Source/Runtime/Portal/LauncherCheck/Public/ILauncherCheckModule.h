// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/** Log categories */
DECLARE_LOG_CATEGORY_EXTERN(LogLauncherCheck, Display, All);

/** Launch options when starting the Launcher */
enum class ELauncherAction : uint32
{
	/** Launch the App after the launcher is started */
	AppLaunch,
	/** Check for an available update to the App after the launcher is started */
	AppUpdateCheck,
	/** Modify the Apps installation - app has to support SD and be installed */
	AppInstaller,
	/** Notify the launcher that installation corruption was detected and that it should verify */
	AppVerifyInstallation
};

/**
 * Interface for the Launcher checking system
 */
class ILauncherCheckModule
	: public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline ILauncherCheckModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ILauncherCheckModule>("LauncherCheck");
	}

	/**
	 * Checks to see if the current app was ran from the Launcher
	 *
	 * @return true, if it was
	 */
	virtual bool WasRanFromLauncher() const = 0;

	/**
	 * Opens the launcher, appending our identifier to the cmdline
	 *
	 * @param Action - The Action we want the launcher to take
	 * @param Payload - (Optional) additional information we want to supply to the action uri
	 * @return true, if it was successful
	 */
	virtual bool RunLauncher(ELauncherAction Action, FString Payload = FString()) const = 0;

public:

	/** Virtual destructor. */
	virtual ~ILauncherCheckModule() = default;
};
