// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LauncherPlatform : ModuleRules
{
    public LauncherPlatform(ReadOnlyTargetRules Target) : base(Target)
    {
		bRequiresPlatformSDK = true;

        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
				"HTTP"
            }
        );

        if (Target.Platform != UnrealTargetPlatform.Linux && Target.Platform != UnrealTargetPlatform.Win64 && Target.Platform != UnrealTargetPlatform.Mac)
        {
            PrecompileForTargets = PrecompileTargetsType.None;
        }
    }
}
