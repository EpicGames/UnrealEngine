// Copyright Epic Games, Inc. All Rights Reserved.

#include "ILauncherPlatform.h"

#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "PlatformHttp.h"

FString FLauncherMisc::GetEncodedExePath()
{
	FString EncodedExePath;

	static const TCHAR* Delims[] = { TEXT("/") };
	static const int32 NumDelims = UE_ARRAY_COUNT(Delims);

	// Get the path to the executable, and encode it ('cos spaces, colons, etc break things)
	FString ExePath = FPlatformProcess::BaseDir();

	// Make sure it's not relative and the slashes are the right way
	ExePath = FPaths::ConvertRelativePathToFull(ExePath);
	ExePath.ReplaceInline(TEXT("\\"), TEXT("/"));

	// Encode the path 'cos symbols like ':',' ' etc are bad
	TArray<FString> ExeFolders;
	ExePath.ParseIntoArray(ExeFolders, Delims, NumDelims);
	for (const FString& ExeFolder : ExeFolders)
	{
		EncodedExePath /= FPlatformHttp::UrlEncode(ExeFolder);
	}
	// Make sure it ends in a slash
	EncodedExePath /= TEXT("");

	return EncodedExePath;
}
