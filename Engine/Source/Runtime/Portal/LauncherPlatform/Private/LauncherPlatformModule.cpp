// Copyright Epic Games, Inc. All Rights Reserved.

#include "LauncherPlatformModule.h"
#if PLATFORM_WINDOWS
#include "Windows/LauncherPlatformWindows.h"
#elif PLATFORM_MAC
#include "Mac/LauncherPlatformMac.h"
#elif PLATFORM_LINUX
#include "Linux/LauncherPlatformLinux.h"
#endif

IMPLEMENT_MODULE( FLauncherPlatformModule, LauncherPlatform );
DEFINE_LOG_CATEGORY(LogLauncherPlatform);

void FLauncherPlatformModule::StartupModule()
{
	LauncherPlatform = new FLauncherPlatform();
}

void FLauncherPlatformModule::ShutdownModule()
{
	if (LauncherPlatform != NULL)
	{
		delete LauncherPlatform;
		LauncherPlatform = NULL;
	}
}