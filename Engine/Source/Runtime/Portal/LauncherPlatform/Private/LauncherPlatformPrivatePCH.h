// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLauncherPlatform, Log, All);

#include "ILauncherPlatform.h"

#if PLATFORM_WINDOWS
#include "Windows/LauncherPlatformWindows.h"
#elif PLATFORM_MAC
#include "Mac/LauncherPlatformMac.h"
#elif PLATFORM_LINUX
#include "Linux/LauncherPlatformLinux.h"
#endif
