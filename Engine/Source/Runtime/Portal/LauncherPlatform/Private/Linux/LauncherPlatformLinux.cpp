// Copyright Epic Games, Inc. All Rights Reserved.

#include "LauncherPlatformLinux.h"

bool FLauncherPlatformLinux::CanOpenLauncher(bool Install)
{
	// TODO: no launcher support at the moment
	return false;
}

bool FLauncherPlatformLinux::OpenLauncher(const FOpenLauncherOptions& Options)
{
	// TODO: support launcher for realz
	return true;
}
