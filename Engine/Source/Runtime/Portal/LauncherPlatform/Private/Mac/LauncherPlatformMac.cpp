// Copyright Epic Games, Inc. All Rights Reserved.

#include "LauncherPlatformMac.h"
#include "Mac/MacPlatformProcess.h"
#include "Mac/MacSystemIncludes.h"
#include "Misc/Paths.h"

bool FLauncherPlatformMac::CanOpenLauncher(bool Install)
{
	FString Path;
	return IsLauncherInstalled() || (Install && GetLauncherInstallerPath(Path));
}

bool FLauncherPlatformMac::OpenLauncher(const FOpenLauncherOptions& Options)
{
	FString LauncherUriRequest = Options.GetLauncherUriRequest();

	// If the launcher is already running, bring it to front
	NSArray* RunningLaunchers = [NSRunningApplication runningApplicationsWithBundleIdentifier: @"com.epicgames.EpicGamesLauncher"];
	if ([RunningLaunchers count] == 0)
	{
		RunningLaunchers = [NSRunningApplication runningApplicationsWithBundleIdentifier: @"com.epicgames.UnrealEngineLauncher"];
	}

	if ([RunningLaunchers count] > 0)
	{
		NSRunningApplication* Launcher = [RunningLaunchers objectAtIndex: 0];
		if ( !Launcher.hidden || Options.bInstall ) // If the launcher is running, but hidden, don't activate on editor startup
		{
			[Launcher activateWithOptions : NSApplicationActivateAllWindows];
			FString Error;
			FPlatformProcess::LaunchURL(*LauncherUriRequest, nullptr, &Error);
		}
		return true;
	}

	if (IsLauncherInstalled())
	{
		FString Error;
		FPlatformProcess::LaunchURL(*LauncherUriRequest, nullptr, &Error);
		return true;
	}

	// Try to install it
	FString InstallerPath;
	if (GetLauncherInstallerPath(InstallerPath))
	{
		FPlatformProcess::LaunchFileInDefaultExternalApplication(*InstallerPath);
		return true;
	}

	return false;
}

bool FLauncherPlatformMac::IsLauncherInstalled() const
{
	// Otherwise search for it...
	NSWorkspace* Workspace = [NSWorkspace sharedWorkspace];
	NSURL* Path = [Workspace URLForApplicationWithBundleIdentifier:@"com.epicgames.EpicGamesLauncher"];
	if( Path )
	{
		return true;
	}

	return false;
}

bool FLauncherPlatformMac::GetLauncherInstallerPath(FString& OutInstallerPath) const
{
	// Check if the installer exists
	FString InstallerPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(*FPaths::EngineDir(), TEXT("Extras/UnrealEngineLauncher/EpicGamesLauncher.dmg")));
	if (FPaths::FileExists(InstallerPath))
	{
		OutInstallerPath = InstallerPath;
		return true;
	}
	InstallerPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(*FPaths::EngineDir(), TEXT("Extras/UnrealEngineLauncher/UnrealEngine.dmg")));
	if (FPaths::FileExists(InstallerPath))
	{
		OutInstallerPath = InstallerPath;
		return true;
	}
	return false;
}
