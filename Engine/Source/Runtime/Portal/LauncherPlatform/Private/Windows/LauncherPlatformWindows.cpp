// Copyright Epic Games, Inc. All Rights Reserved.

#include "LauncherPlatformWindows.h"
#include "LauncherPlatformModule.h"
#include "Windows/WindowsPlatformProcess.h"
#include "Windows/WindowsHWrapper.h"
#include "Misc/Paths.h"

bool FLauncherPlatformWindows::CanOpenLauncher(bool Install)
{
	// Check if the launcher exists, or (optionally) if the installer exists
	FString Path;
	return IsLauncherInstalled() || (Install && GetLauncherInstallerPath(Path));
}

bool FLauncherPlatformWindows::OpenLauncher(const FOpenLauncherOptions& Options)
{	
	// Try to launch it directly
	FString InstallerPath;
	if (IsLauncherInstalled())
	{
		FString LauncherUriRequest = Options.GetLauncherUriRequest();

		FString Error;
		FPlatformProcess::LaunchURL(*LauncherUriRequest, nullptr, &Error);
		if (!Error.IsEmpty())
		{
			UE_LOGF(LogLauncherPlatform, Warning, "FPlatformProcess::LaunchURL had an error: %ls", *Error);
		}
		return true;
	}
	// Otherwise see if we can install it
	else if(Options.bInstall && GetLauncherInstallerPath(InstallerPath))
	{
		FPlatformProcess::LaunchFileInDefaultExternalApplication(*InstallerPath);
		return true;
	}

	return false;
}

bool FLauncherPlatformWindows::IsLauncherInstalled() const
{
	bool bRes = false;

	HKEY hKey;
	if (RegOpenKeyEx(HKEY_CLASSES_ROOT, TEXT("com.epicgames.launcher"), 0, KEY_READ, &hKey) == ERROR_SUCCESS)
	{
		RegCloseKey(hKey);
		bRes = true;
	}

	return bRes;
}

bool FLauncherPlatformWindows::GetLauncherInstallerPath(FString& OutInstallerPath)
{
	FString InstallerPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(*FPaths::EngineDir(), TEXT("Extras/UnrealEngineLauncher/EpicGamesLauncherInstaller.msi")));
	if (FPaths::FileExists(InstallerPath))
	{
		OutInstallerPath = InstallerPath;
		return true;
	}

	return false;
}
