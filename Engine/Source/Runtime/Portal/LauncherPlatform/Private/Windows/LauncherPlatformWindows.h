// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherPlatform.h"

class FLauncherPlatformWindows : public ILauncherPlatform
{
public:
	// ILauncherPlatform Implementation
	virtual bool CanOpenLauncher(bool Install) override;
	virtual bool OpenLauncher(const FOpenLauncherOptions& Options) override;

private:
	bool IsLauncherInstalled() const;
	bool GetLauncherInstallerPath(FString& OutInstallerPath);
};

typedef FLauncherPlatformWindows FLauncherPlatform;
