// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ConfigCacheIni.h"

class FOpenLauncherOptions
{
public:
	FOpenLauncherOptions() {}

	FOpenLauncherOptions(FString InLauncherRelativeUrl)
		: LauncherRelativeUrl(InLauncherRelativeUrl)
	{
		if ( !LauncherRelativeUrl.IsEmpty() )
		{
			bSilent = false;
		}
	}

	FOpenLauncherOptions(bool DoInstall, FString InLauncherRelativeUrl)
		: bInstall(DoInstall)
		, LauncherRelativeUrl(InLauncherRelativeUrl)
	{
		if ( !LauncherRelativeUrl.IsEmpty() || bInstall )
		{
			bSilent = false;
		}
	}

	FString GetLauncherUriRequest() const
	{
		FString LauncherUriRequest;
		int RunningAppTimeout = 60;

		GConfig->GetInt(TEXT("LauncherPlatform"), TEXT("RunningAppTimeout"), RunningAppTimeout, GEngineIni);

		if ( LauncherRelativeUrl.IsEmpty() )
		{
			LauncherUriRequest = TEXT("com.epicgames.launcher:");
		}
		else
		{
			LauncherUriRequest = FString::Printf(TEXT("com.epicgames.launcher://%s"), *LauncherRelativeUrl);
		}

		// Append running app timeout arg.
		if (LauncherUriRequest.Contains("?"))
		{
			LauncherUriRequest += TEXT("&");
		}
		else
		{
			LauncherUriRequest += TEXT("?");
		}
		LauncherUriRequest += FString::Printf(TEXT("runningapptimeout=%d"), RunningAppTimeout);

		// Append silent query string arg.
		if ( bSilent )
		{
			LauncherUriRequest += TEXT("&silent=true");
		}

		// Append action string if present
		if (!Action.IsEmpty())
		{
			LauncherUriRequest += FString::Printf(TEXT("&action=%s"), *Action);
		}

		return LauncherUriRequest;
	}

public:

	bool bInstall = false;
	bool bSilent = true;
	FString Action;
	FString LauncherRelativeUrl;
};


class ILauncherPlatform
{
public:
	/** Virtual destructor */
	virtual ~ILauncherPlatform() = default;

	/**
	 * Determines whether the launcher can be opened.
	 *
	 * @param Install					Whether to include the possibility of installing the launcher in the check.
	 * @return true if the launcher can be opened (or installed).
	 */
	virtual bool CanOpenLauncher(bool Install) = 0;

	/**
	 * Opens the marketplace user interface.
	 *
	 * @param Install					Whether to install the marketplace if it is missing.
	 * @param LauncherRelativeUrl		A url relative to the launcher which you'd like the launcher to navigate to. Empty defaults to the UE homepage
	 * @param CommandLineParams			Optional command to open the launcher with if it is not already open
	 * @return true if the marketplace was opened, false if it is not installed or could not be installed/opened.
	 */
	virtual bool OpenLauncher(const FOpenLauncherOptions& Options) = 0;
};

class FLauncherMisc
{
public:
	/** Return url encoded full path of currently running executable. */
	static LAUNCHERPLATFORM_API FString GetEncodedExePath();
};
