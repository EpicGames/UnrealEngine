// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "ILauncherPlatform.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLauncherPlatform, Log, All);

class FLauncherPlatformModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	static ILauncherPlatform* Get()
	{
		FLauncherPlatformModule& LauncherPlatformModule = FModuleManager::Get().LoadModuleChecked<FLauncherPlatformModule>("LauncherPlatform");
		return LauncherPlatformModule.GetSingleton();
	}

private:
	virtual ILauncherPlatform* GetSingleton() const { return LauncherPlatform; }

	ILauncherPlatform* LauncherPlatform;
};
