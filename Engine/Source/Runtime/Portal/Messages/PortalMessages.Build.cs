// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PortalMessages : ModuleRules
	{
		public PortalMessages(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"MessagingRpc",
					"PortalServices",
				});
		}
	}
}
