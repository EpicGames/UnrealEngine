// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RpcMessage.h"
#include "PortalApplicationWindowMessages.generated.h"

USTRUCT()
struct FPortalApplicationWindowNavigateToRequest
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	FString Url;

	FPortalApplicationWindowNavigateToRequest() { }
	FPortalApplicationWindowNavigateToRequest(const FString& InUrl)
		: Url(InUrl)
	{ }
};


USTRUCT()
struct FPortalApplicationWindowNavigateToResponse
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	bool Result = false;

	FPortalApplicationWindowNavigateToResponse() { }
	FPortalApplicationWindowNavigateToResponse(bool InResult)
		: Result(InResult)
	{ }
};


DECLARE_RPC(FPortalApplicationWindowNavigateTo, bool)
