// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RpcMessage.h"
#include "PortalPackageInstallerMessages.generated.h"


/* Service discovery messages
 *****************************************************************************/

USTRUCT()
struct FPortalPackageInstallerInstallRequest
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	FString AppName;

	UPROPERTY(EditAnywhere, Category="Message")
	FString BuildLabel;

	UPROPERTY(EditAnywhere, Category="Message")
	FString DestinationPath;

	FPortalPackageInstallerInstallRequest() { }
	FPortalPackageInstallerInstallRequest(const FString& InAppName, const FString& InBuildLabel, const FString& InDestinationPath)
		: AppName(InAppName)
		, BuildLabel(InBuildLabel)
		, DestinationPath(InDestinationPath)
	{ }
};


USTRUCT()
struct FPortalPackageInstallerInstallResponse
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	bool Result = false;

	FPortalPackageInstallerInstallResponse() { }
	FPortalPackageInstallerInstallResponse(bool InResult)
		: Result(InResult)
	{ }
};


USTRUCT()
struct FPortalPackageInstallerUninstallRequest
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	FString AppName;

	UPROPERTY(EditAnywhere, Category="Message")
	FString BuildLabel;

	UPROPERTY(EditAnywhere, Category="Message")
	FString InstallationPath;

	UPROPERTY(EditAnywhere, Category="Message")
	bool RemoveUserFiles = false;

	FPortalPackageInstallerUninstallRequest() { }
	FPortalPackageInstallerUninstallRequest(const FString& InAppName, const FString& InBuildLabel, const FString& InInstallationPath, bool InRemoveUserFiles)
		: AppName(InAppName)
		, BuildLabel(InBuildLabel)
		, InstallationPath(InInstallationPath)
		, RemoveUserFiles(InRemoveUserFiles)
	{ }
};


USTRUCT()
struct FPortalPackageInstallerUninstallResponse
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	bool Result = false;

	FPortalPackageInstallerUninstallResponse() { }
	FPortalPackageInstallerUninstallResponse(bool InResult)
		: Result(InResult)
	{ }
};


DECLARE_RPC(FPortalPackageInstallerInstall, bool)
DECLARE_RPC(FPortalPackageInstallerUninstall, bool)
