// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RpcMessage.h"
#include "PortalUserLoginMessages.generated.h"

USTRUCT()
struct FPortalUserLoginPromptUserForSignInRequest
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	FPortalUserLoginPromptUserForSignInRequest()
	{ }
};


USTRUCT()
struct FPortalUserLoginPromptUserForSignInResponse
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Message")
	bool Result;

	FPortalUserLoginPromptUserForSignInResponse()
		: Result(false)
	{ }

	FPortalUserLoginPromptUserForSignInResponse(bool InResult)
		: Result(InResult)
	{ }
};
DECLARE_RPC(FPortalUserLoginPromptUserForSignIn, bool)
