// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Account/IPortalUser.h"
#include "RpcMessage.h"
#include "PortalUserMessages.generated.h"

USTRUCT()
struct FPortalUserGetUserDetailsRequest
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	FPortalUserGetUserDetailsRequest()
	{ }
};


USTRUCT()
struct FPortalUserGetUserDetailsResponse
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Message")
	FPortalUserDetails Result;

	FPortalUserGetUserDetailsResponse() { }
	FPortalUserGetUserDetailsResponse(const FPortalUserDetails& InResult)
		: Result(InResult)
	{ }
};
DECLARE_RPC(FPortalUserGetUserDetails, FPortalUserDetails)


USTRUCT()
struct FPortalUserIsEntitledToItemRequest
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Message")
	FString ItemId;

	UPROPERTY(EditAnywhere, Category = "Message")
	EEntitlementCacheLevelRequest CacheLevel = EEntitlementCacheLevelRequest::Memory;

	FPortalUserIsEntitledToItemRequest()
	{ }

	FPortalUserIsEntitledToItemRequest(
		const FString& InItemId,
		EEntitlementCacheLevelRequest InCacheLevel)
		: ItemId(InItemId)
		, CacheLevel(InCacheLevel)
	{ }
};

USTRUCT()
struct FPortalUserIsEntitledToItemResponse
	: public FRpcMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Message")
	FPortalUserIsEntitledToItemResult Result;

	FPortalUserIsEntitledToItemResponse() 
	{ }
	
	FPortalUserIsEntitledToItemResponse(const FPortalUserIsEntitledToItemResult& InResult)
		: Result(InResult)
	{ }
};
DECLARE_RPC(FPortalUserIsEntitledToItem, FPortalUserIsEntitledToItemResult)
