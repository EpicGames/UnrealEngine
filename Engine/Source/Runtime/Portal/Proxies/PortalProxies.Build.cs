// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PortalProxies : ModuleRules
	{
		public PortalProxies(ReadOnlyTargetRules Target) : base(Target)
		{
            DynamicallyLoadedModuleNames.AddRange(
                new string[] {
                    "Messaging",
                    "MessagingRpc",
                }
            );

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"PortalMessages",
				}
			);

            PrivateIncludePathModuleNames.AddRange(
                new string[] {
                    "MessagingRpc",
                    "PortalServices",
                }
            );

			PrivateIncludePaths.AddRange(
				new string[] {
					"Runtime/Portal/Proxies/Private/Account",
					"Runtime/Portal/Proxies/Private/Application",
					"Runtime/Portal/Proxies/Private/Package",
				}
			);
		}
	}
}
