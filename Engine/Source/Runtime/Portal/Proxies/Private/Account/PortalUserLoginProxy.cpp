// Copyright Epic Games, Inc. All Rights Reserved.

#include "Account/PortalUserLoginProxy.h"
#include "Account/IPortalUserLogin.h"
#include "PortalUserLoginMessages.h"

class FPortalUserLoginProxy
	: public IPortalUserLogin
{
public:

	virtual ~FPortalUserLoginProxy() { }

	virtual bool IsAvailable() const override
	{
		return RpcClient->IsConnected();
	}

public:

	virtual TAsyncResult<bool> PromptUserForSignIn() override
	{
		return RpcClient->Call<FPortalUserLoginPromptUserForSignIn>();
	}


private:

	FPortalUserLoginProxy(const TSharedRef<IMessageRpcClient>& InRpcClient)
		: RpcClient(InRpcClient)
	{ }


private:

	TSharedRef<IMessageRpcClient> RpcClient;

	friend FPortalUserLoginProxyFactory;
};

TSharedRef<IPortalService> FPortalUserLoginProxyFactory::Create(const TSharedRef<IMessageRpcClient>& RpcClient)
{
	return MakeShareable(new FPortalUserLoginProxy(RpcClient));
}
