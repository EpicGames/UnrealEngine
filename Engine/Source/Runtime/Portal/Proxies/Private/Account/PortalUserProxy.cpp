// Copyright Epic Games, Inc. All Rights Reserved.

#include "Account/PortalUserProxy.h"
#include "Account/IPortalUser.h"
#include "PortalUserMessages.h"

class FPortalUserProxy
	: public IPortalUser
{
public:

	virtual ~FPortalUserProxy() { }

	virtual bool IsAvailable() const override
	{
		return RpcClient->IsConnected();
	}

public:

	virtual TAsyncResult<FPortalUserDetails> GetUserDetails() override
	{
		return RpcClient->Call<FPortalUserGetUserDetails>();
	}
	
	virtual TAsyncResult<FPortalUserIsEntitledToItemResult> IsEntitledToItem(const FString& ItemId, EEntitlementCacheLevelRequest CacheLevel) override
	{
		return RpcClient->Call<FPortalUserIsEntitledToItem>(ItemId, CacheLevel);
	}

private:

	FPortalUserProxy(const TSharedRef<IMessageRpcClient>& InRpcClient)
		: RpcClient(InRpcClient)
	{ }

private:

	TSharedRef<IMessageRpcClient> RpcClient;

	friend FPortalUserProxyFactory;
};

TSharedRef<IPortalService> FPortalUserProxyFactory::Create(const TSharedRef<IMessageRpcClient>& RpcClient)
{
	return MakeShareable(new FPortalUserProxy(RpcClient));
}
