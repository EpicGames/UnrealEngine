// Copyright Epic Games, Inc. All Rights Reserved.

#include "Application/PortalApplicationWindowProxy.h"
#include "Application/IPortalApplicationWindow.h"
#include "PortalApplicationWindowMessages.h"

class FPortalApplicationWindowProxy
	: public IPortalApplicationWindow
{
public:

	virtual ~FPortalApplicationWindowProxy() { }

	virtual bool IsAvailable() const override
	{
		return RpcClient->IsConnected();
	}

public:

	virtual TAsyncResult<bool> NavigateTo(const FString& Url) override
	{
		return RpcClient->Call<FPortalApplicationWindowNavigateTo>(Url);
	}

private:

	FPortalApplicationWindowProxy(const TSharedRef<IMessageRpcClient>& InRpcClient)
		: RpcClient(InRpcClient)
	{ }


private:

	TSharedRef<IMessageRpcClient> RpcClient;

	friend FPortalApplicationWindowProxyFactory;
};

TSharedRef<IPortalService> FPortalApplicationWindowProxyFactory::Create(const TSharedRef<IMessageRpcClient>& RpcClient)
{
	return MakeShareable(new FPortalApplicationWindowProxy(RpcClient));
}
