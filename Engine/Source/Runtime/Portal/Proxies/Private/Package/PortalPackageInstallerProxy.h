// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Package/IPortalPackageInstaller.h"
#include "IMessageRpcClient.h"
#include "PortalPackageInstallerMessages.h"


class FPortalPackageInstallerProxy
	: public IPortalPackageInstaller
{
public:

	FPortalPackageInstallerProxy(const TSharedRef<IMessageRpcClient>& InRpc)
		: RpcClient(InRpc)
	{ }

	virtual ~FPortalPackageInstallerProxy() = default;

public:

	virtual TAsyncResult<bool> Install(const FString& Path, const FString& AppName, const FString& BuildLabel) override
	{
		return RpcClient->Call<FPortalPackageInstallerInstall>(Path, AppName, BuildLabel);
	}

	virtual TAsyncResult<bool> Uninstall(const FString& Path, const FString& AppName, const FString& BuildLabel, bool RemoveUserFiles) override
	{
		return RpcClient->Call<FPortalPackageInstallerUninstall>(Path, AppName, BuildLabel, RemoveUserFiles);
	}

private:

	TSharedRef<IMessageRpcClient> RpcClient;
};
