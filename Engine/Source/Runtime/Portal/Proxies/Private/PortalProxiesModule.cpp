// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IMessageRpcClient.h"
#include "Misc/TypeContainer.h"
#include "IPortalServiceProvider.h"

#include "Application/IPortalApplicationWindow.h"
#include "Account/IPortalUser.h"
#include "Account/IPortalUserLogin.h"

#include "Application/PortalApplicationWindowProxy.h"
#include "Account/PortalUserProxy.h"
#include "Account/PortalUserLoginProxy.h"


/**
 * Implements the PortalProxies module.
 */
class FPortalProxiesModule
	: public IPortalServiceProvider
{
public:

	// IModuleInterface interface

	virtual void StartupModule() override 
	{
		ApplicationWindow_SupportedServiceNames.Add(TNameOf<IPortalApplicationWindow>::GetName());
		PortalUser_SupportedServiceNames.Add(TNameOf<IPortalUser>::GetName());
		PortalUserLogin_SupportedServiceNames.Add(TNameOf<IPortalUserLogin>::GetName());
	}
	virtual void ShutdownModule() override { }

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

public:

	// IPortalServiceProvider interface

	virtual TSharedPtr<IPortalService> GetService(const FString& ServiceName, const TSharedRef<FTypeContainer>& Dependencies) override
	{
		TSharedPtr<IMessageRpcClient> RpcClient = Dependencies->GetInstance<IMessageRpcClient>();

		if (!RpcClient.IsValid())
		{
			return nullptr;
		}

		if (ApplicationWindow_SupportedServiceNames.Contains(ServiceName))
		{
			return FPortalApplicationWindowProxyFactory::Create(RpcClient.ToSharedRef());
		}
		else if (PortalUser_SupportedServiceNames.Contains(ServiceName))
		{
			return FPortalUserProxyFactory::Create(RpcClient.ToSharedRef());
		}
		else if (PortalUserLogin_SupportedServiceNames.Contains(ServiceName))
		{
			return FPortalUserLoginProxyFactory::Create(RpcClient.ToSharedRef());
		}

		//Add additional supported proxy services here

		return nullptr;
	}

private:

	TSet<FString> ApplicationWindow_SupportedServiceNames;
	TSet<FString> PortalUser_SupportedServiceNames;
	TSet<FString> PortalUserLogin_SupportedServiceNames;

};


IMPLEMENT_MODULE(FPortalProxiesModule, PortalProxies);
