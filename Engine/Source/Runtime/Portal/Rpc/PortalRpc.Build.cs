// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PortalRpc : ModuleRules
	{
		public PortalRpc(ReadOnlyTargetRules Target) : base(Target)
		{
            DynamicallyLoadedModuleNames.AddRange(
                new string[] {
                }
            );

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                    "MessagingRpc",
				}
			);

            PrivateIncludePathModuleNames.AddRange(
                new string[] {
                    "Messaging",
                    "MessagingCommon",
                }
            );
		}
	}
}
