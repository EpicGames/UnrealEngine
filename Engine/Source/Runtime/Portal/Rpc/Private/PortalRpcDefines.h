// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

/* Private constants
 *****************************************************************************/

/** Defines interval at which the Portal RPC server is being located (in seconds). */
#define PORTAL_RPC_LOCATE_INTERVAL 5.0

/** Defines the time after which Portal RPC servers time out (in seconds). */
#define PORTAL_RPC_LOCATE_TIMEOUT 15.0

