// Copyright Epic Games, Inc. All Rights Reserved.

#include "PortalRpcLocator.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Guid.h"
#include "Containers/Ticker.h"
#include "Misc/EngineVersion.h"
#include "IMessageContext.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "PortalRpcDefines.h"
#include "IPortalRpcLocator.h"
#include "PortalRpcMessages.h"


class FPortalRpcLocatorImpl
	: public IPortalRpcLocator
{
public:

	virtual ~FPortalRpcLocatorImpl()
	{
		FTSTicker::RemoveTicker(TickerHandle);
	}

public:

	// IPortalRpcLocator interface

	virtual const FMessageAddress& GetServerAddress() const override
	{
		return ServerAddress;
	}

	virtual FSimpleDelegate& OnServerLocated() override
	{
		return ServerLocatedDelegate;
	}

	virtual FSimpleDelegate& OnServerLost() override
	{
		return ServerLostDelegate;
	}

private:

	void HandleMessage(const FPortalRpcServer& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
	{
		LastServerResponse = FDateTime::UtcNow();

		FMessageAddress NewServerAddress;

		if (FMessageAddress::Parse(Message.ServerAddress, NewServerAddress) && (NewServerAddress != ServerAddress))
		{
			ServerAddress = NewServerAddress;
			ServerLocatedDelegate.ExecuteIfBound();
		}
	}

	bool HandleTicker(float DeltaTime)
	{
        QUICK_SCOPE_CYCLE_COUNTER(STAT_FPortalRpcLocatorImple_HandleTicker);

		if (ServerAddress.IsValid() && ((FDateTime::UtcNow() - LastServerResponse).GetTotalSeconds() > PORTAL_RPC_LOCATE_TIMEOUT))
		{
			ServerAddress.Invalidate();
			ServerLostDelegate.ExecuteIfBound();
		}

		// @todo sarge: implement actual product GUID
		MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FPortalRpcLocateServer>(FGuid(), EngineVersion, MacAddress, UserId), EMessageScope::Network);

		return true;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPortalRpcLocatorImpl()
		: EngineVersion(FEngineVersion::Current().ToString())
		, MacAddress(FPlatformMisc::GetMacAddressString())
		, UserId(FPlatformProcess::UserName(false))
		, LastServerResponse(FDateTime::MinValue())
	{
		MessageEndpoint = FMessageEndpoint::Builder("FPortalRpcLocator")
			.Handling<FPortalRpcServer>(this, &FPortalRpcLocatorImpl::HandleMessage);

		// this can return null in shipping builds, when MessageBus is disabled
		if (MessageEndpoint)
		{
			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FPortalRpcLocatorImpl::HandleTicker), PORTAL_RPC_LOCATE_INTERVAL);
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:

	const FString EngineVersion;
	const FString MacAddress;
	const FString UserId;

	/** Time at which the RPC server last responded. */
	FDateTime LastServerResponse;

	/** Message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** The message address of the located RPC server, or invalid if no server available. */
	FMessageAddress ServerAddress;

	/** A delegate that is executed when an RPC server has been located. */
	FSimpleDelegate ServerLocatedDelegate;

	/** A delegate that is executed when the RPC server has been lost. */
	FSimpleDelegate ServerLostDelegate;

	/** Handle to the registered ticker. */
	FTSTicker::FDelegateHandle TickerHandle;

	friend FPortalRpcLocatorFactory;
};


TSharedRef<IPortalRpcLocator> FPortalRpcLocatorFactory::Create()
{
	return MakeShareable(new FPortalRpcLocatorImpl());
}
