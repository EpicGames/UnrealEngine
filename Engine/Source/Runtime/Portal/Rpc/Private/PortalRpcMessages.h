// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "PortalRpcMessages.generated.h"


USTRUCT()
struct FPortalRpcLocateServer
{
	GENERATED_USTRUCT_BODY()

	/** The product's unique identifier. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid ProductId;

	/** The product's version string. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString ProductVersion;

	/** The mac address of the host. */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString HostMacAddress;

	/** The user identification for the host. */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString HostUserId;

	/** Default constructor. */
	FPortalRpcLocateServer() { }

	/** Create and initialize a new instance. */
	FPortalRpcLocateServer(const FGuid& InProductId, const FString& InProductVersion, const FString& InHostMacAddress, const FString& InHostUserId)
		: ProductId(InProductId)
		, ProductVersion(InProductVersion)
		, HostMacAddress(InHostMacAddress)
		, HostUserId(InHostUserId)
	{ }
};


USTRUCT()
struct FPortalRpcServer
{
	GENERATED_USTRUCT_BODY()

	/** The RPC server's message address as a string. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString ServerAddress;

	/** Default constructor. */
	FPortalRpcServer() { }

	/** Create and initialize a new instance. */
	FPortalRpcServer(const FString& InServerAddress)
		: ServerAddress(InServerAddress)
	{ }
};
