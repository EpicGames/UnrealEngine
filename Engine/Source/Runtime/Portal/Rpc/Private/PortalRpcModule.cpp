// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IPortalRpcModule.h"
#include "PortalRpcLocator.h"
#include "PortalRpcResponder.h"
#include "PortalRpcServer.h"

class FPortalRpcModule
	: public IPortalRpcModule
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

public:

	virtual TSharedRef<IPortalRpcLocator> CreateLocator() override
	{
		return FPortalRpcLocatorFactory::Create();
	}

	virtual TSharedRef<IPortalRpcResponder> CreateResponder() override
	{
		return FPortalRpcResponderFactory::Create();
	}

	virtual TSharedRef<IPortalRpcServer> CreateServer() override
	{
		return FPortalRpcServerFactory::Create();
	}

};

IMPLEMENT_MODULE(FPortalRpcModule, PortalRpc);
