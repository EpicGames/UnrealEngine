// Copyright Epic Games, Inc. All Rights Reserved.

#include "PortalRpcResponder.h"
#include "IMessageContext.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "IPortalRpcResponder.h"
#include "HAL/PlatformProcess.h"
#include "IPortalRpcServer.h"
#include "PortalRpcMessages.h"

class FPortalRpcResponderImpl
	: public IPortalRpcResponder
{
public:

	virtual ~FPortalRpcResponderImpl() { }

public:

	// IPortalRpcResponder interface
	virtual FOnPortalRpcLookup& OnLookup() override
	{
		return LookupDelegate;
	}

private:

	FPortalRpcResponderImpl(
		const FString& InMyHostMacAddress,
		const FString& InMyHostUserId)
		: MyHostMacAddress(InMyHostMacAddress)
		, MyHostUserId(InMyHostUserId)
	{
		MessageEndpoint = FMessageEndpoint::Builder("FPortalRpcResponder")
			.Handling<FPortalRpcLocateServer>(this, &FPortalRpcResponderImpl::HandleMessage);

		if (MessageEndpoint.IsValid())
		{
			MessageEndpoint->Subscribe<FPortalRpcLocateServer>();
		}
	}

private:

	void HandleMessage(const FPortalRpcLocateServer& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
	{
		if (!LookupDelegate.IsBound())
		{
			return;
		}

		if (Message.HostMacAddress != MyHostMacAddress && Message.HostUserId != MyHostUserId)
		{
			return;
		}

		const FString ProductKey = Message.ProductId.ToString() + Message.ProductVersion;
		TSharedPtr<IPortalRpcServer> Server = Servers.FindRef(ProductKey);

		if (!Server.IsValid())
		{
			Server = LookupDelegate.Execute(ProductKey);
		}

		if (Server.IsValid())
		{
			Server->ConnectTo(Context->GetSender());
		}
	}


private:

	const FString MyHostMacAddress;
	const FString MyHostUserId;

	/** A delegate that is executed when a look-up for an RPC server occurs. */
	FOnPortalRpcLookup LookupDelegate;

	/** Message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Holds the existing RPC servers. */
	TMap<FString, TSharedPtr<IPortalRpcServer>> Servers;

	friend FPortalRpcResponderFactory;
};

TSharedRef<IPortalRpcResponder> FPortalRpcResponderFactory::Create()
{
	// @todo: this need to use GetLoginId, but we need to deprecate this functionality over time.
	// eventually, when GetMacAddressString is removed from the codebase, this coude will need to be removed also.
	// In the meantime, it needs to handle BOTH the old Mac address and FPlatformMisc::GetLoginId as a way of recognizing the local machine.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FString Identifier = FPlatformMisc::GetMacAddressString();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return MakeShareable(new FPortalRpcResponderImpl(Identifier, FPlatformProcess::UserName(false)));
}
