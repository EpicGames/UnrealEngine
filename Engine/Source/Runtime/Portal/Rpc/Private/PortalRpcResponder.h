// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IPortalRpcResponder;

class FPortalRpcResponderFactory
{
public:
	static TSharedRef<IPortalRpcResponder> Create();
};
