// Copyright Epic Games, Inc. All Rights Reserved.

#include "PortalRpcServer.h"
#include "IPortalRpcServer.h"
#include "PortalRpcMessages.h"
#include "MessageEndpoint.h"
#include "MessageRpcServer.h"

class FPortalRpcServerImpl
	: public FMessageRpcServer
	, public IPortalRpcServer
{
public:

	virtual void ConnectTo(const FMessageAddress& Address) const
	{
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FPortalRpcServer>(GetAddress().ToString()), Address);
	}

	virtual IMessageRpcServer* GetMessageServer()
	{
		return static_cast<IMessageRpcServer*>(this);
	}

public:

	virtual ~FPortalRpcServerImpl() { }

private:

	FPortalRpcServerImpl()
		: FMessageRpcServer()
	{}

private:

	friend FPortalRpcServerFactory;
};

TSharedRef<IPortalRpcServer> FPortalRpcServerFactory::Create()
{
	return MakeShareable(new FPortalRpcServerImpl());
}
