// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IPortalRpcServer;

class FPortalRpcServerFactory
{
public:
	static TSharedRef<IPortalRpcServer> Create();
};
