// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FMessageAddress;

/**
 * Interface for Portal RPC server locators.
 */
class IPortalRpcLocator
{
public:

	/**
	 * Get the address of the located Portal RPC server.
	 *
	 * @return The RPC server's message address, or invalid if no server found.
	 * @see OnServerLocated, OnServerLost
	 */
	virtual const FMessageAddress& GetServerAddress() const = 0;

public:

	/**
	 * Get a delegate that is executed when an RPC server has been located.
	 *
	 * @return The delegate.
	 * @see GetServerAddress, OnServerLost
	 */
	virtual FSimpleDelegate& OnServerLocated() = 0;
	
	/**
	 * Get a delegate that is executed when an RPC server has been located.
	 *
	 * @return The delegate.
	 * @see GetServerAddress, OnServerLocated
	 */
	virtual FSimpleDelegate& OnServerLost() = 0;

public:

	/** Virtual destructor. */
	virtual ~IPortalRpcLocator() = default;
};
