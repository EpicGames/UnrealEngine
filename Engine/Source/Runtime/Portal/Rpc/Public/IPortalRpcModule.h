// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class IPortalRpcLocator;
class IPortalRpcResponder;
class IPortalRpcServer;

/**
 * Interface for the PortalRpc module.
 */
class IPortalRpcModule
	: public IModuleInterface
{
public:

	virtual TSharedRef<IPortalRpcLocator> CreateLocator() = 0;
	virtual TSharedRef<IPortalRpcResponder> CreateResponder() = 0;
	virtual TSharedRef<IPortalRpcServer> CreateServer() = 0;
};
