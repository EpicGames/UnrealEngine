// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IPortalRpcServer;

DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IPortalRpcServer>, FOnPortalRpcLookup, const FString& /*ProductKey*/)


/**
 * Interface for Portal RPC responders.
 */
class IPortalRpcResponder
{
public:

	/** Get a delegate that is executed when a look-up for an RPC server occurs. */
	virtual FOnPortalRpcLookup& OnLookup() = 0;

public:

	/** Virtual destructor. */
	virtual ~IPortalRpcResponder() = default;
};
