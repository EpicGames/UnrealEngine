// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IMessageRpcServer;
struct FMessageAddress;

/**
 * Interface for Portal RPC server
 */
class IPortalRpcServer
{
public:

	/**
	 * Sends a FPortalRpcServer response to the specified address, effectively
	 * allowing the two services to now communicate with each other directly.
	 */
	virtual void ConnectTo(const FMessageAddress& Address) const = 0;

	virtual IMessageRpcServer* GetMessageServer() = 0;

public:

	/** Virtual destructor. */
	virtual ~IPortalRpcServer() = default;
};
