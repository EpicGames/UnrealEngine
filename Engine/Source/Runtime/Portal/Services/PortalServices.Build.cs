// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PortalServices : ModuleRules
	{
		public PortalServices(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
				}
			);

            PublicDefinitions.Add("WITH_PORTAL_SERVICES=1");
		}
	}
}
