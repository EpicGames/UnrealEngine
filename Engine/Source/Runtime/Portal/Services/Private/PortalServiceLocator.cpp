// Copyright Epic Games, Inc. All Rights Reserved.

#include "PortalServiceLocator.h"
#include "Misc/WildcardString.h"
#include "Modules/ModuleManager.h"
#include "IPortalServiceLocator.h"
#include "IPortalServiceProvider.h"

class FPortalServiceLocatorImpl
	: public IPortalServiceLocator
{
public:

	~FPortalServiceLocatorImpl() { }

public:

	// IPortalServiceLocator interface

	virtual void Configure(const FString& ServiceName, const FWildcardString& ProductId, const FName& ServiceModule) override
	{
		TArray<FConfigEntry>& Entries = Configuration.FindOrAdd(ServiceName);
		FConfigEntry Entry;
		{
			Entry.ProductId = ProductId;
			Entry.ServiceModule = ServiceModule;
		}

		Entries.Add(Entry);
	}

	virtual TSharedPtr<IPortalService> GetService(const FString& ServiceName, const FString& ProductId) override
	{
		TArray<FConfigEntry>& Entries = Configuration.FindOrAdd(ServiceName);

		for (FConfigEntry& Entry : Entries)
		{
			if (!Entry.ProductId.IsMatch(ProductId))
			{
				continue;
			}

			if (Entry.ServiceInstance.IsValid())
			{
				return Entry.ServiceInstance;
			}

			auto ServiceProvider = FModuleManager::LoadModulePtr<IPortalServiceProvider>(Entry.ServiceModule);

			if (ServiceProvider == nullptr)
			{
				continue;
			}

			Entry.ServiceInstance = ServiceProvider->GetService(ServiceName, ServiceDependencies.ToSharedRef());

			if (Entry.ServiceInstance.IsValid())
			{
				return Entry.ServiceInstance;
			}
		}

		return nullptr;
	}

private:
	
	/**
	 * Create and initialize a new instance.
	 *
	 * @param InServiceDependencies A type container for optional service dependencies.
	 */
	FPortalServiceLocatorImpl(const TSharedRef<FTypeContainer>& InServiceDependencies)
		: ServiceDependencies(InServiceDependencies)
	{ }


private:

	struct FConfigEntry
	{
		FWildcardString ProductId;
		TSharedPtr<IPortalService> ServiceInstance;
		FName ServiceModule;
	};

	/** Holds the service configuration entries. */
	TMap<FString, TArray<FConfigEntry>> Configuration;

	/** Optional service dependencies. */
	TSharedPtr<FTypeContainer> ServiceDependencies;

	friend FPortalServiceLocatorFactory;
};

TSharedRef<IPortalServiceLocator> FPortalServiceLocatorFactory::Create(const TSharedRef<FTypeContainer>& ServiceDependencies)
{
	return MakeShareable(new FPortalServiceLocatorImpl(ServiceDependencies));
}
