// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class FTypeContainer;
class IPortalServiceLocator;

class FPortalServiceLocatorFactory
{
public:
	static TSharedRef<IPortalServiceLocator> Create(const TSharedRef<FTypeContainer>& ServiceDependencies);
};
