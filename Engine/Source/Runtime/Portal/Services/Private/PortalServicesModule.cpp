// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IPortalServicesModule.h"
#include "PortalServiceLocator.h"


/**
 * Implements the PortalServices module.
 */
class FPortalServicesModule
	: public IPortalServicesModule
{
public:

	/** Virtual destructor. */
	virtual ~FPortalServicesModule() { }

public:

	// IPortalServicesModule interface

	virtual TSharedRef<IPortalServiceLocator> CreateLocator(const TSharedRef<FTypeContainer>& ServiceDependencies) override
	{
		return FPortalServiceLocatorFactory::Create(ServiceDependencies);
	}

public:

	// IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
	virtual bool SupportsDynamicReloading() override { return true; }
};


IMPLEMENT_MODULE(FPortalServicesModule, PortalServices);
