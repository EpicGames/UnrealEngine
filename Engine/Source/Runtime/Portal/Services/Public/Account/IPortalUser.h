// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Async/AsyncResult.h"
#include "IPortalService.h"
#include "IPortalUser.generated.h"

/** 
 * The basic data for the last or currently signed-in user.
 */
USTRUCT()
struct FPortalUserDetails
{
	GENERATED_USTRUCT_BODY()

	/** The users epic games account display name */
	UPROPERTY(EditAnywhere, Category = "User")
	FText DisplayName;

	/** The users epic games account email address */
	UPROPERTY(EditAnywhere, Category = "User")
	FText Email;

	/** The users real name attached to their epic games account */
	UPROPERTY(EditAnywhere, Category = "User")
	FText RealName;

	/** Whether this user is currently signed-in to the portal */
	UPROPERTY(EditAnywhere, Category = "User")
	bool IsSignedIn;

	FPortalUserDetails()
		: DisplayName()
		, Email()
		, RealName()
		, IsSignedIn(false)
	{ }
};

/** 
 * The available options for the different levels of caches available when
 * requesting information about user entitlements.
 *
 * Disk cache level includes Memory cache level.
 */
UENUM()
enum class EEntitlementCacheLevelRequest : uint8
{
	/** 
	 * Allow reference to entitlements cached in-memory for this session of the portal. 
	 * These are periodically updated. This is the recommended level for entitlement checking.
	 */
	Memory = 1,

	/** 
	 * Allow reference to entitlements cached on disk between sessions of the portal. 
	 * This cache is updated regularly while a user is signed-in.  If no user is signed-in
	 * then disk cached entitlements are still used from the last signed-in user's session.
	 */
	Disk = 2
};

/** 
 * Used to delineate which cache an entitlement check result was retrieved from.
 */
UENUM()
enum class EEntitlementCacheLevelRetrieved : uint8
{
	None = 0,

	/** 
	 * Allow reference to entitlements cached in-memory for this session of the portal. 
	 * These are periodically updated. This is the recommended level for entitlement checking.
	 */
	Memory = 1,

	/** 
	 * Allow reference to entitlements cached on disk between sessions of the portal. 
	 * This cache is updated regularly while a user is signed-in.  If no user is signed-in
	 * then disk cached entitlements are still used from the last signed-in user's session.
	 */
	Disk = 2
};

USTRUCT()
struct FPortalUserIsEntitledToItemResult
{
	GENERATED_USTRUCT_BODY()

	/** The item id that was checked for an active entitlement */
	UPROPERTY(EditAnywhere, Category = "Entitlement")
	FString ItemId;

	/** Whether this user is entitled to the item id */
	UPROPERTY(EditAnywhere, Category = "Entitlement")
	bool IsEntitled;

	/** How fresh this entitlement information is */
	UPROPERTY(EditAnywhere, Category = "Entitlement")
	EEntitlementCacheLevelRetrieved RetrievedFromCacheLevel;

	FPortalUserIsEntitledToItemResult()
		: ItemId()
		, IsEntitled(false)
		, RetrievedFromCacheLevel(EEntitlementCacheLevelRetrieved::None)
	{ }

	FPortalUserIsEntitledToItemResult(
		const FString& InItemId,
		const bool InIsEntitled,
		const EEntitlementCacheLevelRetrieved InRetrievedFromCacheLevel)
		: ItemId(InItemId)
		, IsEntitled(InIsEntitled)
		, RetrievedFromCacheLevel(InRetrievedFromCacheLevel)
	{ }
};

/**
 * Interface for the Portal application's user services.
 * Specializes in readonly requests for information about the last or currently signed-in user.
 */
class IPortalUser
	: public IPortalService
{
public:

	/**
	 * Requests the details of the last or currently signed in user
	 */
	virtual TAsyncResult<FPortalUserDetails> GetUserDetails() = 0;

	/**
	 * Returns whether the use is entitled to the specified Item Id.
	 * 
	 * @param ItemId    The guid for the catalog item to check the entitled state for
	 * @param CacheLevel    The level at which to allow referencing cached data
	 *
	 * Behavior will vary based on the entitlement cache level specified and whether the user is
	 * signed-in or not, or if the user is signed-in but latest entitlements have yet to be retrieved.
	 * See documentation about the entitlement cache levels for details.
	 *
	 * In the case where there is no cached entitlement data and the latest entitlements have yet to be 
	 * retrieved, then false will be returned. The Portal will not wait for the entitlements to update
	 * before returning a response. This state can be detected when the RetrievedFromCacheLevel is None.
	 */
	virtual TAsyncResult<FPortalUserIsEntitledToItemResult> IsEntitledToItem(const FString& ItemId, EEntitlementCacheLevelRequest CacheLevel) = 0;

public:

	virtual ~IPortalUser() = default;
};

Expose_TNameOf(IPortalUser);
