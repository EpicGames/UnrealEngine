// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Async/AsyncResult.h"
#include "IPortalService.h"

/**
 * Interface for the Portal application's user login services.
 * Specializes in actions related to signing a user in and out of the portal.
 */
class IPortalUserLogin
	: public IPortalService
{
public:

	/**
	 * Requests that the portal come to the front and ask the user to sign-in.
	 *
	 * If a user is already signed into the Portal then a prompt may ask the existing user if they 
	 * want to sign-out, especially if they have on going installations or downloads. The user may opt to not
	 * sign-out if this happens.
	 *
	 * @return whether the portal successfully received and handled your request. Note this doesn't
	 * mean that the user signed-in successfully or even that the existing user signed-out. It simply
	 * means that the Portal attempted to prompt the user for sign-in.
	 */
	virtual TAsyncResult<bool> PromptUserForSignIn() = 0;


public:

	virtual ~IPortalUserLogin() = default;
};

Expose_TNameOf(IPortalUserLogin);
