// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Async/AsyncResult.h"
#include "IPortalService.h"

/**
 * Interface for the Portal application's marketplace services.
 */
class IPortalApplicationWindow
	: public IPortalService
{
public:

	/**
	 * Requests that the Portal navigate the user to the specified Url
	 *
	 * @return true on success, false otherwise.
	 */
	virtual TAsyncResult<bool> NavigateTo(const FString& Url) = 0;

public:

	/** Virtual destructor. */
	virtual ~IPortalApplicationWindow() = default;
};
Expose_TNameOf(IPortalApplicationWindow);
