// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IPortalService
{
public:

	/** Virtual destructor. */
	virtual ~IPortalService() = default;

	virtual bool IsAvailable() const = 0;
};
