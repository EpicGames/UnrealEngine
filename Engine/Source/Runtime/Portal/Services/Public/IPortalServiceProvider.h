// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class FTypeContainer;
class IPortalService;

class IPortalServiceProvider
	: public IModuleInterface
{
public:

	/**
	 * Get a service instance of the specified type. 
	 *
	 * @param ServiceName The name of the service type to get.
	 * @param Dependencies Optional service dependencies.
	 * @return The service instance, or nullptr if unavailable.
	 */
	virtual TSharedPtr<IPortalService> GetService(const FString& ServiceName, const TSharedRef<FTypeContainer>& Dependencies) = 0;
};
