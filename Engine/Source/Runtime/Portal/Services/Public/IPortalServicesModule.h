// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class FTypeContainer;
class IPortalServiceLocator;

class IPortalServicesModule
	: public IModuleInterface
{
public:

	/**
	 * Create a locator for Portal services.
	 *
	 * @param ServiceDependencies Any dependencies that the services may need.
	 * @return A service locator.
	 */
	virtual TSharedRef<IPortalServiceLocator> CreateLocator(const TSharedRef<FTypeContainer>& ServiceDependencies) = 0;

public:

	/** Virtual destructor. */
	virtual ~IPortalServicesModule() = default;
};
