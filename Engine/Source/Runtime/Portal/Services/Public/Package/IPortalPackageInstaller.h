// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncResult.h"
#include "IPortalService.h"


/**
 * Interface for package installer services.
 */
class IPortalPackageInstaller
	: public IPortalService
{
public:

	/**
	 * Install the specified package using the given request object.
	 *
	 * @param AppName The name of the application to install.
	 * @param BuildLabel The application's build label.
	 * @return The result of the task.
	 * @see UninstallPackage
	 */
	virtual TAsyncResult<bool> Install(const FString& Path, const FString& AppName, const FString& BuildLabel) = 0;

	/**
	 * Attempts to uninstall the specified package using the given request object.
	 *
	 * @param Path The path at which the package is installed.
	 * @param AppName The name of the application to uninstall.
	 * @param BuildLabel The application's build label.
	 * @param RemoveUserFiles Whether user created files should be removed as well.
	 * @return The result of the task.
	 * @see InstallPackage
	 */
	virtual TAsyncResult<bool> Uninstall(const FString& Path, const FString& AppName, const FString& BuildLabel, bool RemoveUserFiles) = 0;

public:

	/** Virtual destructor. */
	virtual ~IPortalPackageInstaller() = default;
};


Expose_TNameOf(IPortalPackageInstaller)
